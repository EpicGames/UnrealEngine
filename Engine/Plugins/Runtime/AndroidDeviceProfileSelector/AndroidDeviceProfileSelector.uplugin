{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Android Device Profile Selector",
	"Description" : "Android Device Profile Selector used show selection of device profiles on hardware",
	"Category" : "Device Profile Selectors",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AndroidDeviceProfileSelector",
			"Type": "Editor",
			"LoadingPhase" : "None"
		},
		{
			"Name": "AndroidDeviceProfileCommandlets",
			"Type": "Editor",
			"LoadingPhase": "None"
		},
		{
			"Name" : "AndroidDeviceProfileSelectorRuntime",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" :
			[
				"Android"
			]
		}
	]
}
