// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System.IO;
using UnrealBuildTool;

public class AndroidDeviceProfileCommandlets : ModuleRules
{
	public AndroidDeviceProfileCommandlets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add("Developer/Android/AndroidDeviceDetection/Public");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Json",
				"JsonUtilities",
				"AndroidDeviceProfileSelector",
				"PIEPreviewDeviceSpecification"
			}
			);
	}
}
