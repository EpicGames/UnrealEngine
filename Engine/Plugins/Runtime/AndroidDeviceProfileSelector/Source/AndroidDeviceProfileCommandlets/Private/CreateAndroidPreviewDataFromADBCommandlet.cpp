// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateAndroidPreviewDataFromADBCommandlet.h"
#include "PIEPreviewDeviceSpecification.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "Serialization/Csv/CsvParser.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateAndroidPreviewDataFromADBCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCreateAndroidPreviewDataFromADB, Log, All);

// Config rules entries. (used to refine hw, chipset and GPU names based on known info of the hardware)
struct FConfigRuleChipsetEntry
{
	FString Hardware;
	FString Chipset;
	FString GPUFamily;
};
typedef TArray<FConfigRuleChipsetEntry> FConfigRules;

// Load the device's json file and update it with config rules's HW information.
void UpdateJSONWithConfigrules(const FConfigRules& HardwareToGPUName, const FString& JsonPath)
{
	TSharedPtr<FJsonObject> JsonRootObject;
	FString Json;
	if (FFileHelper::LoadFileToString(Json, *JsonPath))
	{
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(Json);
		FJsonSerializer::Deserialize(JsonReader, JsonRootObject);
	}
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	// Update GPUFamily field with known values for specific hardware
	if (JsonRootObject.IsValid() && FJsonObjectConverter::JsonAttributesToUStruct(JsonRootObject->Values, FPIEPreviewDeviceSpecifications::StaticStruct(), &DeviceSpecs, 0, 0))
	{
		for (const FConfigRuleChipsetEntry& ConfRule : HardwareToGPUName)
		{
			if (ConfRule.Hardware.Contains(DeviceSpecs.AndroidProperties.Hardware) || DeviceSpecs.AndroidProperties.Hardware.Contains(ConfRule.Hardware))
			{
				DeviceSpecs.AndroidProperties.Hardware = ConfRule.Hardware;
				DeviceSpecs.AndroidProperties.GPUFamily = ConfRule.GPUFamily;
				DeviceSpecs.AndroidProperties.Chipset = ConfRule.Chipset;
				// keep going, later conf rules take pri.
			}
		}
	}

	// Write the updated JSon object back
	TSharedPtr<FJsonObject> JsonObject = FJsonObjectConverter::UStructToJsonObject<FPIEPreviewDeviceSpecifications>(DeviceSpecs);
	// remove IOS field
	JsonObject->RemoveField(TEXT("IOSProperties"));
	JsonObject->RemoveField(TEXT("switchProperties"));

	// serialize the JSon object to string
	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

	// Save file to disk
	FFileHelper::SaveStringToFile(OutputString, *JsonPath);
}


int32 UCreateAndroidPreviewDataFromADBCommandlet::Main(const FString& RawCommandLine)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;
	ParseCommandLine(*RawCommandLine, Tokens, Switches, Params);

	FString ParamDeviceSpecsFolder = Params.FindRef(TEXT("DeviceSpecsFolder"));
	FString ConfigRulesFile = Params.FindRef(TEXT("ConfigRules"));

	FConfigRules HardwareToGPUName;
	if (!ConfigRulesFile.IsEmpty())
	{
		TArray<FString> Results;

		FFileHelper::LoadFileToStringArrayWithPredicate(Results, *ConfigRulesFile, [](const FString& Line) {	return Line.StartsWith(TEXT("chipset:"));});

		for (const FString& Line : Results)
		{
			FCsvParser lineParser(Line.RightChop(8));
			const FCsvParser::FRows& Rows = lineParser.GetRows();
			if (Rows.Num() == 1)
			{
				if (Rows[0].Num() == 7)
				{
					FConfigRuleChipsetEntry NewEntry;
					NewEntry.Hardware = FString(Rows[0][0]).TrimStartAndEnd().TrimQuotes();
					NewEntry.Chipset = FString(Rows[0][2]).TrimStartAndEnd().TrimQuotes();
					NewEntry.GPUFamily = FString(Rows[0][3]).TrimStartAndEnd().TrimQuotes();
					HardwareToGPUName.Add(NewEntry);
				}
			}
		}
	}

	IAndroidDeviceDetection* DeviceDetection;
	DeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection").GetAndroidDeviceDetection();
	DeviceDetection->Initialize(TEXT("ANDROID_HOME"),
#if PLATFORM_WINDOWS
		TEXT("platform-tools\\adb.exe"),
#else
		TEXT("platform-tools/adb"),
#endif
		TEXT("shell getprop"), true);

	// Decrease device detection poll time.
	IConsoleVariable* const CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Android.DeviceDetectionPollInterval"));
	if (CVar)
	{
		CVar->Set(1);
	}

	TSet<FString> AlreadyExported;
	do
	{
		{
			FScopeLock ExportLock(DeviceDetection->GetDeviceMapLock());

			const TMap<FString, FAndroidDeviceInfo>& Devices = DeviceDetection->GetDeviceMap();
			for (auto DeviceTuple : Devices)
			{
				const FAndroidDeviceInfo& DeviceInfo = DeviceTuple.Value;
				FString DeviceName = FString::Printf(TEXT("%s_%s(OS%s)"), *DeviceInfo.DeviceBrand, *DeviceInfo.Model, *DeviceInfo.HumanAndroidVersion);
				if (!AlreadyExported.Find(DeviceName))
				{
					FString ExportPath = ParamDeviceSpecsFolder / (DeviceName + TEXT(".json"));
					UE_LOGF(LogCreateAndroidPreviewDataFromADB, Log, "Found new device '%ls' Exporting to %ls", *DeviceName, *ExportPath);
					DeviceDetection->ExportDeviceProfile(ExportPath, DeviceTuple.Key);
					AlreadyExported.Add(DeviceName);

					UpdateJSONWithConfigrules(HardwareToGPUName, ExportPath);
				}
			}
		}
		FPlatformProcess::Sleep(1.0f);
	} while (true);
}

