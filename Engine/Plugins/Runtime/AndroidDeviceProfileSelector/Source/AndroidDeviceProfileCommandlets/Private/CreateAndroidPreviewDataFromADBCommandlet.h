// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "CreateAndroidPreviewDataFromADBCommandlet.generated.h"

/*
* This commandlet runs in an infinite loop. It waits for new devices to be plugged in (or to become visible to adb)
* When a new device is encountered it is queried and a new .json containing it's particulars is saved.
* The json file is then usable by editor for device preview and also for the android device profile matching system.
* Usage commandline:
*
* <game> -run=AndroidDeviceDetection.CreateAndroidPreviewDataFromADB -ConfigRules=[path to configrules.txt] -DeviceSpecsFolder=[Path to json output folder, e.g."\Engine\Content\Editor\PIEPreviewDeviceSpecs\Android"]
 */
UCLASS()
class UCreateAndroidPreviewDataFromADBCommandlet
	: public UCommandlet
{
	GENERATED_BODY()
public:

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& Params) override;

	//~ End UCommandlet Interface
private:

};
