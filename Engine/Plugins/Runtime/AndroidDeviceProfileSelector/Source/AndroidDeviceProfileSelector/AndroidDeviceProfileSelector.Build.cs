// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System.IO;
using System.Linq;
using UnrealBuildTool;

public class AndroidDeviceProfileSelector : ModuleRules
{
	public AndroidDeviceProfileSelector(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "AndroidDPS";

		// get a value from .ini (ConfigFile attribute doesn't work in Build.cs files it seems)
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, Target.ProjectFile != null ? Target.ProjectFile.Directory : null, Target.Platform);
		string SecretGuid;
		Ini.GetString("AndroidDPSBuildSettings", "SecretGuid", out SecretGuid);
		if (!string.IsNullOrEmpty(SecretGuid))
		{
			PrivateDefinitions.Add("HASH_PEPPER_SECRET_GUID="+ SecretGuid);
		}

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
			}
			);

		if(Target.Type == TargetType.Editor)
		{
			if (!Target.GlobalDefinitions.Contains("WITH_ANDROID_DEVICE_DETECTION=0"))
			{
				PrivateDefinitions.Add("WITH_ANDROID_DEVICE_DETECTION=1");
				PrivateDependencyModuleNames.Add("AndroidDeviceDetection");
			}

			PrivateDependencyModuleNames.Add("PIEPreviewDeviceSpecification");
			PublicDependencyModuleNames.Add("Json");
			PublicDependencyModuleNames.Add("JsonUtilities");
			PrivateDependencyModuleNames.Add("RHI");
			PrivateDependencyModuleNames.Add("ApplicationCore");
		}
	}
}
