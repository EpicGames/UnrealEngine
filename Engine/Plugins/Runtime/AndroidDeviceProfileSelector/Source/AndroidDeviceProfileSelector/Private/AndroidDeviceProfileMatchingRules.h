// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AndroidDeviceProfileMatchingRules.generated.h"

UENUM()
enum ESourceType : int
{
	SRC_PreviousRegexMatch,
	SRC_GpuFamily,
	SRC_GlVersion,
	SRC_AndroidVersion,
	SRC_DeviceMake,
	SRC_DeviceModel,
	SRC_DeviceBuildNumber,
	SRC_VulkanVersion,
	SRC_UsingHoudini,
	SRC_VulkanAvailable,
	SRC_CommandLine,
	SRC_Hardware,
	SRC_Chipset,
	SRC_ConfigRuleVar,
	SRC_HMDSystemName,
	SRC_SM5Available,
	SRC_VKQuality,
	SRC_MAX,
};

UENUM()
enum ECompareType : int
{
	CMP_Equal,
	CMP_Less,
	CMP_LessEqual,
	CMP_Greater,
	CMP_GreaterEqual,
	CMP_NotEqual,
	CMP_Regex,
	CMP_EqualIgnore,
	CMP_LessIgnore,
	CMP_LessEqualIgnore,
	CMP_GreaterIgnore,
	CMP_GreaterEqualIgnore,
	CMP_NotEqualIgnore,
	CMP_Hash,
	CMP_MAX,
};

USTRUCT()
struct FProfileMatchItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TEnumAsByte<ESourceType> SourceType=ESourceType::SRC_MAX;

	UPROPERTY()
	TEnumAsByte<ECompareType> CompareType=ECompareType::CMP_MAX;

	UPROPERTY()
	FString MatchString;
};

USTRUCT()
struct FProfileMatch
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Profile;
	
	UPROPERTY()
	TArray<FProfileMatchItem> Match;
};

UCLASS(config = DeviceProfiles)
class UAndroidDeviceProfileMatchingRules : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// Use Android .ini files in the editor
	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return TEXT("Android");
	}

	/** Array of rules to match */
	UPROPERTY(EditAnywhere, config, Category = "Matching Rules")
	TArray<FProfileMatch> MatchProfile;
};
