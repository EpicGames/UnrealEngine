// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidDeviceProfileSelector.h"
#include "AndroidDeviceProfileMatchingRules.h"
#include "AndroidJavaSurfaceViewDevices.h"
#include "Internationalization/Regex.h"
#include "Misc/CommandLine.h"

#if PLATFORM_ANDROID
#include "Android/AndroidPlatformMisc.h"
#endif

UAndroidDeviceProfileMatchingRules::UAndroidDeviceProfileMatchingRules(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAndroidJavaSurfaceViewDevices::UAndroidJavaSurfaceViewDevices(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

static UAndroidDeviceProfileMatchingRules* GetAndroidDeviceProfileMatchingRules()
{
#if !UE_WITH_CONSTINIT_UOBJECT
	// We need to initialize the class early as device profiles need to be evaluated before ProcessNewlyLoadedUObjects can be called.
	extern UClass* Z_Construct_UClass_UAndroidDeviceProfileMatchingRules(ETypeConstructPhase);
	CreatePackage(UAndroidDeviceProfileMatchingRules::StaticPackage());
	Z_Construct_UClass_UAndroidDeviceProfileMatchingRules(ETypeConstructPhase::Outer);
#endif // !UE_WITH_CONSTINIT_UOBJECT

	// Get the default object which will has the values from DeviceProfiles.ini
	UAndroidDeviceProfileMatchingRules* Rules = Cast<UAndroidDeviceProfileMatchingRules>(UAndroidDeviceProfileMatchingRules::StaticClass()->GetDefaultObject());
	check(Rules);
	return Rules;
}

TMap<FString, FString> FAndroidDeviceProfileSelector::SelectorProperties;

void FAndroidDeviceProfileSelector::VerifySelectorParams()
{
	auto GetSelectorPropertiesValue = [](const FStringView& InStringView)
		{
			return SelectorProperties.FindByHash(GetTypeHash(InStringView), InStringView);
		};
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_GPUFamily));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_GLVersion));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_AndroidVersion));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_DeviceMake));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_DeviceModel));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_DeviceBuildNumber));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_VulkanVersion));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_UsingHoudini));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_VulkanAvailable));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_Hardware));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_Chipset));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_HMDSystemName));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_TotalPhysicalGB));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_SM5Available));
	check(GetSelectorPropertiesValue(FAndroidProfileSelectorSourceProperties::SRC_VKQuality));
}

FString FAndroidDeviceProfileSelector::FindMatchingProfile(const FString& FallbackProfileName)
{
	FString OutProfileName = FallbackProfileName;
	FString CommandLine = FCommandLine::Get();

	auto GetSelectorPropertiesValueChecked = [](const FStringView& InStringView)
		{
			return SelectorProperties.FindByHashChecked(GetTypeHash(InStringView), InStringView);
		};
	FString GPUFamily = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_GPUFamily);
	FString GLVersion = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_GLVersion);
	FString AndroidVersion = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_AndroidVersion);
	FString DeviceMake = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_DeviceMake);
	FString DeviceModel = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_DeviceModel);
	FString DeviceBuildNumber = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_DeviceBuildNumber);
	FString VulkanVersion = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_VulkanVersion);
	FString UsingHoudini = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_UsingHoudini);
	FString VulkanAvailable = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_VulkanAvailable);
	FString Hardware = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_Hardware);
	FString Chipset = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_Chipset);
	FString TotalPhysicalGB = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_TotalPhysicalGB);
	FString HMDSystemName = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_HMDSystemName);
	FString SM5Available = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_SM5Available);
	FString VKQuality = GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_VKQuality);

	for (const FProfileMatch& Profile : GetAndroidDeviceProfileMatchingRules()->MatchProfile)
	{
		FString PreviousRegexMatch;
		bool bFoundMatch = true;
		for (const FProfileMatchItem& Item : Profile.Match)
		{
			FString ConfigRuleString;
			const FString* SourceString = nullptr;
			FString MatchString = Item.MatchString;
			switch (Item.SourceType)
			{
			case ESourceType::SRC_PreviousRegexMatch:
				SourceString = &PreviousRegexMatch;
				break;
			case ESourceType::SRC_GpuFamily:
				SourceString = &GPUFamily;
				break;
			case ESourceType::SRC_GlVersion:
				SourceString = &GLVersion;
				break;
			case ESourceType::SRC_AndroidVersion:
				SourceString = &AndroidVersion;
				break;
			case ESourceType::SRC_DeviceMake:
				SourceString = &DeviceMake;
				break;
			case ESourceType::SRC_DeviceModel:
				SourceString = &DeviceModel;
				break;
			case ESourceType::SRC_DeviceBuildNumber:
				SourceString = &DeviceBuildNumber;
				break;
			case ESourceType::SRC_VulkanVersion:
				SourceString = &VulkanVersion;
				break;
			case ESourceType::SRC_UsingHoudini:
				SourceString = &UsingHoudini;
				break;
			case ESourceType::SRC_VulkanAvailable:
				SourceString = &VulkanAvailable;
				break;
			case ESourceType::SRC_CommandLine:
				SourceString = &CommandLine;
				break;
			case ESourceType::SRC_Hardware:
				SourceString = &Hardware;
				break;
			case ESourceType::SRC_Chipset:
				SourceString = &Chipset;
				break;
			case ESourceType::SRC_HMDSystemName:
				SourceString = &HMDSystemName;
				break;
			case ESourceType::SRC_SM5Available:
				SourceString = &SM5Available;
				break;
			case ESourceType::SRC_VKQuality:
				SourceString = &VKQuality;
				break;
			case ESourceType::SRC_ConfigRuleVar:
				{
					// expected Matchstring contents for configrulevar is "configrule_varname|matchstring"
					// sourcestring is set to the configrule variable content. 
					// sourcestring will be an empty string if configrule_varname is not found.
					FString VariableValueMatchString;
					FString ConfigRuleVarName;
					SourceString = &ConfigRuleString;
					if (MatchString.Split(TEXT("|"), &ConfigRuleVarName, &VariableValueMatchString))
					{
						MatchString = VariableValueMatchString;
#if PLATFORM_ANDROID
						const FString* ConfigRuleVar = FPlatformMisc::GetConfigRulesVariable(ConfigRuleVarName);
#elif WITH_EDITOR
						const FString* ConfigRuleVar = SelectorProperties.Find(FString::Printf(TEXT("SRC_ConfigRuleVar[%s]"), *ConfigRuleVarName));
#endif
						if (ConfigRuleVar)
						{
							ConfigRuleString = *ConfigRuleVar;
						}
					}					
					break;
				}
			default:
				continue;
			}

			const bool bNumericOperands = SourceString->IsNumeric() && MatchString.IsNumeric();

			switch (Item.CompareType)
			{
			case CMP_Equal:
				if (Item.SourceType == SRC_CommandLine) 
				{
					if (!FParse::Param(*CommandLine, *MatchString))
					{
						bFoundMatch = false;
					}
				}
				else
				{
					if (*SourceString != MatchString)
					{
						bFoundMatch = false;
					}
				}
				break;
			case CMP_Less:
				if ((bNumericOperands && FCString::Atof(**SourceString) >= FCString::Atof(*MatchString)) || (!bNumericOperands && *SourceString >= MatchString))
				{
					bFoundMatch = false;
				}
				break;
			case CMP_LessEqual:
				if ((bNumericOperands && FCString::Atof(**SourceString) > FCString::Atof(*MatchString)) || (!bNumericOperands && *SourceString > MatchString))
				{
					bFoundMatch = false;
				}
				break;
			case CMP_Greater:
				if ((bNumericOperands && FCString::Atof(**SourceString) <= FCString::Atof(*MatchString)) || (!bNumericOperands && *SourceString <= MatchString))
				{
					bFoundMatch = false;
				}
				break;
			case CMP_GreaterEqual:
				if ((bNumericOperands && FCString::Atof(**SourceString) < FCString::Atof(*MatchString)) || (!bNumericOperands && *SourceString < MatchString))
				{
					bFoundMatch = false;
				}
				break;
			case CMP_NotEqual:
				if (Item.SourceType == SRC_CommandLine)
				{
					if (FParse::Param(*CommandLine, *MatchString))
					{
						bFoundMatch = false;
					}
				}
				else
				{
					if (*SourceString == MatchString)
					{
						bFoundMatch = false;
					}
				}
				break;
			case CMP_EqualIgnore:
				if (SourceString->ToLower() != MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_LessIgnore:
				if (SourceString->ToLower() >= MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_LessEqualIgnore:
				if (SourceString->ToLower() > MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_GreaterIgnore:
				if (SourceString->ToLower() <= MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_GreaterEqualIgnore:
				if (SourceString->ToLower() < MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_NotEqualIgnore:
				if (SourceString->ToLower() == MatchString.ToLower())
				{
					bFoundMatch = false;
				}
				break;
			case CMP_Regex:
				{
					const FRegexPattern RegexPattern(MatchString);
					FRegexMatcher RegexMatcher(RegexPattern, *SourceString);
					if (RegexMatcher.FindNext())
					{
						PreviousRegexMatch = RegexMatcher.GetCaptureGroup(1);
					}
					else
					{
						bFoundMatch = false;
					}
				}
			break;
				case CMP_Hash:
				{
					// Salt string is concatenated onto the end of the input text.
					// For example the input string "PhoneModel" with salt "Salt" and pepper "Pepper" can be computed with
					// % printf "PhoneModelSaltPepper" | openssl dgst -sha1 -hex
					// resulting in d9e5cbd6b0e4dba00edd9de92cf64ee4c3f3a2db and would be stored in the matching rules as 
					// "Salt|d9e5cbd6b0e4dba00edd9de92cf64ee4c3f3a2db". Salt is optional.
					FString MatchHashString;
					FString SaltString;
					if (!MatchString.Split(TEXT("|"), &SaltString, &MatchHashString))
					{
						MatchHashString = MatchString;
					}
					FString HashInputString = *SourceString + SaltString
#ifdef HASH_PEPPER_SECRET_GUID
						+ HASH_PEPPER_SECRET_GUID.ToString()
#endif
						;

					FSHAHash SourceHash;
					FSHA1::HashBuffer(TCHAR_TO_ANSI(*HashInputString), HashInputString.Len(), SourceHash.Hash);
					if (SourceHash.ToString() != MatchHashString.ToUpper())
					{
						bFoundMatch = false;
					}
				}
				break;
			default:
				bFoundMatch = false;
			}

			if (!bFoundMatch)
			{
				break;
			}
		}

		if (bFoundMatch)
		{
			OutProfileName = Profile.Profile;
			break;
		}
	}
	return OutProfileName;
}

int32 FAndroidDeviceProfileSelector::GetNumProfiles()
{
	return GetAndroidDeviceProfileMatchingRules()->MatchProfile.Num();
}
