// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidDeviceProfileSelectorModule.h"
#include "AndroidDeviceProfileSelector.h"
#include "Modules/ModuleManager.h"
#if WITH_EDITOR
#include "GenericPlatform/GenericConfigRules.h"
#if WITH_ANDROID_DEVICE_DETECTION
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#endif
#include "PIEPreviewDeviceSpecification.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DataDrivenShaderPlatformInfo.h"

#endif
#include "Misc/ConfigCacheIni.h"
#include "Android/AndroidPlatformProperties.h"

IMPLEMENT_MODULE(FAndroidDeviceProfileSelectorModule, AndroidDeviceProfileSelector);

DEFINE_LOG_CATEGORY_STATIC(LogAndroidDPSelector, Log, All)

void FAndroidDeviceProfileSelectorModule::StartupModule()
{
}

void FAndroidDeviceProfileSelectorModule::ShutdownModule()
{
}

const FString FAndroidDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	// We are not expecting this module to have GetRuntimeDeviceProfileName called directly.
	// Android ProfileSelectorModule runtime is now in FAndroidDeviceProfileSelectorRuntimeModule.
	// Use GetDeviceProfileName.
	checkNoEntry();
	return FString();
}

#if WITH_EDITOR
void FAndroidDeviceProfileSelectorModule::ExportDeviceParametersToJson(FString& FolderLocation, TArray<FString>& OutExportedFiles)
{
#if WITH_ANDROID_DEVICE_DETECTION
	IAndroidDeviceDetection* DeviceDetection;
	DeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection").GetAndroidDeviceDetection();
	DeviceDetection->Initialize(TEXT("ANDROID_HOME"),
#if PLATFORM_WINDOWS
		TEXT("platform-tools\\adb.exe"),
#else
		TEXT("platform-tools/adb"),
#endif
		TEXT("shell getprop"), true);

	TSet<FString> AlreadyExported;

	{
		FScopeLock ExportLock(DeviceDetection->GetDeviceMapLock());

		const TMap<FString, FAndroidDeviceInfo>& Devices = DeviceDetection->GetDeviceMap();
		if (Devices.Num() == 0)
		{
			UE_LOGF(LogAndroidDPSelector, Warning, "ExportDeviceParametersToJson - No device detected. Check that $ANDROID_HOME is set and USB debugging is enabled on the device.");
		}
		else
		{
			for (auto DeviceTuple : Devices)
			{
				const FAndroidDeviceInfo& DeviceInfo = DeviceTuple.Value;
				if (!DeviceInfo.DeviceBrand.IsEmpty() && !DeviceInfo.Model.IsEmpty())
				{
					FString DeviceName = FString::Printf(TEXT("%s_%s(OS%s)"), *DeviceInfo.DeviceBrand, *DeviceInfo.Model, *DeviceInfo.HumanAndroidVersion);
					if (!AlreadyExported.Find(DeviceName))
					{
						FString ExportPath = FolderLocation / (DeviceName + TEXT(".json"));
						OutExportedFiles.Add(ExportPath);
						DeviceDetection->ExportDeviceProfile(ExportPath, DeviceTuple.Key);
						AlreadyExported.Add(DeviceName);
					}
				}
			}
		}
	}
	FPlatformProcess::Sleep(1.0f);
#endif
}

bool FAndroidDeviceProfileSelectorModule::CanExportDeviceParametersToJson()
{
#if WITH_ANDROID_DEVICE_DETECTION
	return true;
#else
	return false;
#endif
}

static bool GetDeviceSpecsFromJson(const FString& JsonLocation, FPIEPreviewDeviceSpecifications& OutDeviceSpecs)
{
	static TMap<const FString, FPIEPreviewDeviceSpecifications> FileLocationToDeviceSpec;
	if (FPIEPreviewDeviceSpecifications* CachedDeviceSpecs = FileLocationToDeviceSpec.Find(JsonLocation))
	{
		OutDeviceSpecs = *CachedDeviceSpecs;
		return true;
	}

	TSharedPtr<FJsonObject> JsonRootObject;
	FString Json;

	if (FFileHelper::LoadFileToString(Json, *JsonLocation))
	{
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(Json);
		FJsonSerializer::Deserialize(JsonReader, JsonRootObject);
	}

	if (JsonRootObject.IsValid() && FJsonObjectConverter::JsonAttributesToUStruct(JsonRootObject->Values, FPIEPreviewDeviceSpecifications::StaticStruct(), &OutDeviceSpecs, 0, 0))
	{
		if (OutDeviceSpecs.AndroidProperties.Version == ANDROID_DEVICE_PROPERTIES_VERSION)
		{
			FileLocationToDeviceSpec.Add(JsonLocation, OutDeviceSpecs);
		}
		return true;
	}

	return false;
}

bool FAndroidDeviceProfileSelectorModule::IsJsonCompatible(const FString& JsonLocation)
{
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	if (GetDeviceSpecsFromJson(JsonLocation, DeviceSpecs))
	{
		if (DeviceSpecs.AndroidProperties.Version > ANDROID_DEVICE_PROPERTIES_VERSION)
		{
			UE_LOGF(LogAndroidDPSelector, Log, "Json %ls was saved with a newer version than this Engine Version supports(Found v%d, Expected v%d), please update your Json by connecting the same device that generated this Json and call Generate Preview Json.",
				*JsonLocation, DeviceSpecs.AndroidProperties.Version, ANDROID_DEVICE_PROPERTIES_VERSION);
			return false;
		}
		else if (DeviceSpecs.DevicePlatform != EPIEPreviewDeviceType::Android)
		{
			UE_LOGF(LogAndroidDPSelector, Log, "Json %ls was saved with a different DevicePlatform than Android. ", *JsonLocation);
			return false;
		}
	}
	else
	{
		return false;
	}

	return true;
}

static void MakeOutdatedDeviceSpecCompatible(FPIEPreviewDeviceSpecifications& OutDeviceSpecs)
{

}

void FAndroidDeviceProfileSelectorModule::GetDeviceParametersFromJson(FString& JsonLocation, TMap<FString, FString>& OutDeviceParameters)
{
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	if (GetDeviceSpecsFromJson(JsonLocation, DeviceSpecs))
	{
		if (DeviceSpecs.AndroidProperties.Version < ANDROID_DEVICE_PROPERTIES_VERSION)
		{
			MakeOutdatedDeviceSpecCompatible(DeviceSpecs);
			UE_LOGF(LogAndroidDPSelector, Log, "Json %ls is using an Outdated Json Version(Found v%d, Expected v%d), please update your Json by connecting the same device that generated this Json and call Generate Preview Json.",
				*JsonLocation, DeviceSpecs.AndroidProperties.Version, ANDROID_DEVICE_PROPERTIES_VERSION);
		}
		FPIEAndroidDeviceProperties& AndroidProperties = DeviceSpecs.AndroidProperties;
		OutDeviceParameters.Add(TEXT("SRC_GPUFamily"), AndroidProperties.GPUFamily);
		OutDeviceParameters.Add(TEXT("SRC_GLVersion"), AndroidProperties.GLVersion);
		OutDeviceParameters.Add(TEXT("SRC_VulkanAvailable"), AndroidProperties.VulkanAvailable ? "true" : "false");
		OutDeviceParameters.Add(TEXT("SRC_VulkanVersion"), AndroidProperties.VulkanVersion);
		OutDeviceParameters.Add(TEXT("SRC_AndroidVersion"), AndroidProperties.AndroidVersion);
		OutDeviceParameters.Add(TEXT("SRC_DeviceMake"), AndroidProperties.DeviceMake);
		OutDeviceParameters.Add(TEXT("SRC_DeviceModel"), AndroidProperties.DeviceModel);
		OutDeviceParameters.Add(TEXT("SRC_DeviceBuildNumber"), AndroidProperties.DeviceBuildNumber);
		OutDeviceParameters.Add(TEXT("SRC_UsingHoudini"), AndroidProperties.UsingHoudini ? "true" : "false");
		OutDeviceParameters.Add(TEXT("SRC_Hardware"), AndroidProperties.Hardware);
		OutDeviceParameters.Add(TEXT("SRC_Chipset"), AndroidProperties.Chipset);
		OutDeviceParameters.Add(TEXT("SRC_TotalPhysicalGB"), AndroidProperties.TotalPhysicalGB);
		OutDeviceParameters.Add(TEXT("SRC_HMDSystemName"), TEXT(""));
		OutDeviceParameters.Add(TEXT("SRC_SM5Available"), AndroidProperties.SM5Available ? "true" : "false");
		OutDeviceParameters.Add(TEXT("SRC_VKQuality"), TEXT(""));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_ResolutionX), FString::Printf(TEXT("%d"), DeviceSpecs.ResolutionX));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_ResolutionY), FString::Printf(TEXT("%d"), DeviceSpecs.ResolutionY));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_InsetsLeft), FString::Printf(TEXT("%f"), DeviceSpecs.InsetsLeft));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_InsetsTop), FString::Printf(TEXT("%f"), DeviceSpecs.InsetsTop));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_InsetsRight), FString::Printf(TEXT("%f"), DeviceSpecs.InsetsRight));
		OutDeviceParameters.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_InsetsBottom), FString::Printf(TEXT("%f"), DeviceSpecs.InsetsBottom));
	}
	else
	{
		UE_LOGF(LogAndroidDPSelector, Log, "Json %ls failed to load", *JsonLocation);
		return;
	}

	// Initialize the ConfigRules for the Preview Device.
	FString ConfigRulesFile = FPaths::ProjectDir() + TEXT("Build/Android/configrules.txt");
	TArray<uint8> ConfigRulesData;
	TMap<FString, FString> PredefinedVariables(OutDeviceParameters);
	if (FFileHelper::LoadFileToArray(ConfigRulesData, *ConfigRulesFile))
	{
		const TMap<FString, FString>& PreviewConfigRules = FGenericConfigRules::ParseConfigRules(ConfigRulesData, MoveTemp(PredefinedVariables));
		// Initialize the DeviceParams for the Preview Device.
		for (const TPair<FString, FString>& Pair : PreviewConfigRules)
		{
			const FString& VariableName = Pair.Key;
			const FString& VariableValue = Pair.Value;
			OutDeviceParameters.Add(FString::Printf(TEXT("SRC_ConfigRuleVar[%s]"), *VariableName), *VariableValue);
		}
	}
}

const EShaderPlatform FAndroidDeviceProfileSelectorModule::GetPreviewShaderPlatform()
{
	// ensure SelectorProperties does actually contain our parameters
	check(FAndroidDeviceProfileSelector::GetSelectorProperties().Num() > 0);
	FString ProfileName = ANDROID_DEFAULT_DEVICE_PROFILE_NAME;
	ProfileName = FAndroidDeviceProfileSelector::FindMatchingProfile(ProfileName);
	EShaderPlatform ParentShaderPlatform = SP_OPENGL_ES3_1_ANDROID;
	TMap<FName, FString> DeviceProfileCVars = UDeviceProfileManager::Get().GatherDeviceProfileCVars(ProfileName, UDeviceProfileManager::EDeviceProfileMode::DPM_CacheValues);
	if (const FString* DisableVulkanSupport = DeviceProfileCVars.Find(TEXT("r.Android.DisableVulkanSupport")))
	{
		bool bIsDisableVulkanSupport = FCString::Atoi(**DisableVulkanSupport) == 1;
		if (!bIsDisableVulkanSupport)
		{
			ParentShaderPlatform = SP_VULKAN_ES3_1_ANDROID;
		}
	}

	EShaderPlatform PreviewPlatform = SP_NumPlatforms;
	for (int32 PlatformIndex = SP_CUSTOM_PLATFORM_FIRST; PlatformIndex < SP_NumPlatforms; PlatformIndex++)
	{
		EShaderPlatform PreviewPlatformToCheck = EShaderPlatform(PlatformIndex);
		if (FDataDrivenShaderPlatformInfo::IsValid(PreviewPlatformToCheck) &&
			FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(PreviewPlatformToCheck) &&
			FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(PreviewPlatformToCheck) == ParentShaderPlatform)
		{
			PreviewPlatform = PreviewPlatformToCheck;
			break;
		}
	}
	return PreviewPlatform;
}

#endif

const FString FAndroidDeviceProfileSelectorModule::GetDeviceProfileName()
{
	FString ProfileName = ANDROID_DEFAULT_DEVICE_PROFILE_NAME;

	// ensure SelectorProperties does actually contain our parameters
	check(FAndroidDeviceProfileSelector::GetSelectorProperties().Num() > 0);

	UE_LOGF(LogAndroidDPSelector, Log, "Checking %d rules from DeviceProfile ini file.", FAndroidDeviceProfileSelector::GetNumProfiles() );
	UE_LOGF(LogAndroidDPSelector, Log, "  Default profile: %ls", *ProfileName);
	for (const TTuple<FString,FString>& MapIt : FAndroidDeviceProfileSelector::GetSelectorProperties())
	{
		UE_LOGF(LogAndroidDPSelector, Log, "  %ls: %ls", *MapIt.Key, *MapIt.Value);
	}

	// Use override from ConfigRules if set
	const FString* ConfigProfile = FAndroidDeviceProfileSelector::GetSelectorProperties().Find(TEXT("Profile"));
	if (ConfigProfile != nullptr)
	{
		ProfileName = *ConfigProfile;
		UE_LOGF(LogAndroidDPSelector, Log, "Using ConfigRules Profile: [%ls]", *ProfileName);
	}
	else
	{
		// Find a match with the DeviceProfiles matching rules
		ProfileName = FAndroidDeviceProfileSelector::FindMatchingProfile(ProfileName);
		UE_LOGF(LogAndroidDPSelector, Log, "Selected Device Profile: [%ls]", *ProfileName);
	}

	UE_LOGF(LogAndroidDPSelector, Log, "Selected Device Profile: [%ls]", *ProfileName);

	return ProfileName;
}

bool FAndroidDeviceProfileSelectorModule::GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT)
{
	if (const FString* Found = FAndroidDeviceProfileSelector::GetSelectorProperties().Find(PropertyType.ToString()))
	{
		PropertyValueOUT = *Found;
		return true;
	}
	// Special case for non-existent config rule variables
	// they should return true and a value of '[null]'
	// this prevents configrule issues from throwing errors.
	if (PropertyType.ToString().StartsWith(TEXT("SRC_ConfigRuleVar[")))
	{
		PropertyValueOUT = TEXT("[null]");
		return true;
	}

	return false;
}

void FAndroidDeviceProfileSelectorModule::SetSelectorProperties(const TMap<FString, FString>& SelectorPropertiesIn)
{
	auto GetSelectorPropertiesValueChecked = [SelectorPropertiesIn](const FStringView& InStringView)
		{
			return SelectorPropertiesIn.FindByHashChecked(GetTypeHash(InStringView), InStringView);
		};
	const int32 ResolutionX= FCString::Atoi(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_ResolutionX));
	const int32 ResolutionY = FCString::Atoi(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_ResolutionY));
	const float InsetsLeft = FCString::Atof(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_InsetsLeft));
	const float InsetsTop = FCString::Atof(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_InsetsTop));
	const float InsetsRight = FCString::Atof(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_InsetsRight));
	const float InsetsBottom = FCString::Atof(*GetSelectorPropertiesValueChecked(FAndroidProfileSelectorSourceProperties::SRC_InsetsBottom));
	
	FString Orientation;
	bool bNeedPortrait = false;
	GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("Orientation"), Orientation, GEngineIni);
	if (Orientation.ToLower().Equals("portrait") || Orientation.ToLower().Equals("reverseportrait") || Orientation.ToLower().Equals("sensorportrait"))
	{
		bNeedPortrait = true;
	}

	if (bNeedPortrait)
	{
		SafeZones.X = InsetsLeft;
		SafeZones.Y = InsetsTop;
		SafeZones.Z = InsetsRight;
		SafeZones.W = InsetsBottom;
		ConstrainedAspectRatio = float(ResolutionX) / float(ResolutionY);
	}
	else
	{
		SafeZones.X = InsetsTop;
		SafeZones.Y = InsetsRight;
		SafeZones.Z = InsetsBottom;
		SafeZones.W = InsetsLeft;
		ConstrainedAspectRatio = float(ResolutionY) / float(ResolutionX);
	}
	FAndroidDeviceProfileSelector::SetSelectorProperties(SelectorPropertiesIn);
}
