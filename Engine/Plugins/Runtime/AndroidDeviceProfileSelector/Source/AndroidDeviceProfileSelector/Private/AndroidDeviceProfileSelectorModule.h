// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the Android Device Profile Selector module.
 * Used to emulate an android device's IDeviceProfileSelectorModule behaviour.
 */
class FAndroidDeviceProfileSelectorModule
	: public IDeviceProfileSelectorModule
{
public:

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetDeviceProfileName() override;
	virtual const FString GetRuntimeDeviceProfileName() override;
#if WITH_EDITOR
	virtual void ExportDeviceParametersToJson(FString& FolderLocation, TArray<FString>& OutExportedFiles) override;
	virtual bool CanExportDeviceParametersToJson() override;
	virtual bool IsJsonCompatible(const FString& JsonLocation) override;
	virtual void GetDeviceParametersFromJson(FString& JsonLocation, TMap<FString, FString>& OutDeviceParameters) override;
	virtual bool CanGetDeviceParametersFromJson() override { return true; }
	virtual const EShaderPlatform GetPreviewShaderPlatform() override;
#endif
	// Set the device parameters this selector will use.
	virtual void SetSelectorProperties(const TMap<FString, FString>& SelectorProperties) override;
	virtual bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) override;
	//~ End IDeviceProfileSelectorModule Interface

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	/**
	 * Virtual destructor.
	 */
	virtual ~FAndroidDeviceProfileSelectorModule() = default;
};
