// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"

#define UE_API ANDROIDDEVICEPROFILESELECTOR_API

namespace FAndroidProfileSelectorSourceProperties
{
	static FStringView SRC_GPUFamily(TEXT("SRC_GPUFamily"));
	static FStringView SRC_GLVersion(TEXT("SRC_GLVersion"));
	static FStringView SRC_VulkanAvailable(TEXT("SRC_VulkanAvailable"));
	static FStringView SRC_VulkanVersion(TEXT("SRC_VulkanVersion"));
	static FStringView SRC_AndroidVersion(TEXT("SRC_AndroidVersion"));
	static FStringView SRC_DeviceMake(TEXT("SRC_DeviceMake"));
	static FStringView SRC_DeviceModel(TEXT("SRC_DeviceModel"));
	static FStringView SRC_DeviceBuildNumber(TEXT("SRC_DeviceBuildNumber"));
	static FStringView SRC_UsingHoudini(TEXT("SRC_UsingHoudini"));
	static FStringView SRC_Hardware(TEXT("SRC_Hardware"));
	static FStringView SRC_Chipset(TEXT("SRC_Chipset"));
	static FStringView SRC_HMDSystemName(TEXT("SRC_HMDSystemName"));
	static FStringView SRC_TotalPhysicalGB(TEXT("SRC_TotalPhysicalGB"));
	static FStringView SRC_SM5Available(TEXT("SRC_SM5Available"));
	static FStringView SRC_ResolutionX(TEXT("SRC_ResolutionX"));
	static FStringView SRC_ResolutionY(TEXT("SRC_ResolutionY"));
	static FStringView SRC_InsetsLeft(TEXT("SRC_InsetsLeft"));
	static FStringView SRC_InsetsTop(TEXT("SRC_InsetsTop"));
	static FStringView SRC_InsetsRight(TEXT("SRC_InsetsRight"));
	static FStringView SRC_InsetsBottom(TEXT("SRC_InsetsBottom"));
	static FStringView SRC_VKQuality(TEXT("SRC_VKQuality"));
};

class FAndroidDeviceProfileSelector
{
	// Container of various device properties used for device profile matching.
	static UE_API TMap<FString, FString> SelectorProperties;
	static UE_API void VerifySelectorParams();
public:

	static UE_API FString FindMatchingProfile(const FString& FallbackProfileName);
	static UE_API int32 GetNumProfiles();
	static void SetSelectorProperties(const TMap<FString, FString>& Params) { SelectorProperties = Params; VerifySelectorParams(); }
	static const TMap<FString, FString>& GetSelectorProperties() { return SelectorProperties; }
};

#undef UE_API
