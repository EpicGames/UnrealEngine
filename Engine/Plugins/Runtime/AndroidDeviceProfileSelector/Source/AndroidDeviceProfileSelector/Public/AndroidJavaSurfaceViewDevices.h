// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AndroidJavaSurfaceViewDevices.generated.h"

USTRUCT()
struct FJavaSurfaceViewDevice
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Manufacturer;

	UPROPERTY()
	FString Model;
};

UCLASS(config = DeviceProfiles)
class UAndroidJavaSurfaceViewDevices : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Array of devices that require the java view scaling workaround */
	UPROPERTY(EditAnywhere, config, Category = "Matching Rules")
	TArray<FJavaSurfaceViewDevice> SurfaceViewDevices;
};
