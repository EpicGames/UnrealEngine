// Copyright Epic Games, Inc. All Rights Reserved.AndroidDeviceProfileSelector

#include "AndroidDeviceProfileSelectorRuntimeModule.h"
#include "AndroidDeviceProfileSelectorRuntime.h"
#include "Templates/Casts.h"
#include "Internationalization/Regex.h"
#include "Modules/ModuleManager.h"
#include "AndroidDeviceProfileSelectorRuntime.h"
#include "AndroidDeviceProfileSelector.h"
#include "AndroidJavaSurfaceViewDevices.h"
#include "IHeadMountedDisplayModule.h"

IMPLEMENT_MODULE(FAndroidDeviceProfileSelectorRuntimeModule, AndroidDeviceProfileSelectorRuntime);

void FAndroidDeviceProfileSelectorRuntimeModule::StartupModule()
{
}

void FAndroidDeviceProfileSelectorRuntimeModule::ShutdownModule()
{
}

// Build the selector params for the current device.
static const TMap<FString,FString>& GetDeviceSelectorParams()
{
	static bool bInitialized = false;
	static TMap<FString, FString> AndroidParams;
	if(!bInitialized)
	{
		bInitialized = true;
		auto GetParam = [](const FString& DefaultParam, const TCHAR* ConfRuleVarName)
		{
#if PLATFORM_ANDROID
			if (const FString* ConfRuleVarValue = FPlatformMisc::GetConfigRulesVariable(ConfRuleVarName))
			{
				return *ConfRuleVarValue;
			}
#endif
			return DefaultParam;
		};

#if !(PLATFORM_ANDROID_X86 || PLATFORM_ANDROID_X64)
		// Not running an Intel libUnreal.so with Houdini library present means we're emulated
		bool bUsingHoudini = (access("/system/lib/libhoudini.so", F_OK) != -1);
#else
		bool bUsingHoudini = false;
#endif

		FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
		// this is used in the same way as PlatformMemoryBucket
		// which on Android has a different rounding algo. See GenericPlatformMemory::GetMemorySizeBucket.
		uint64 MemoryBucketRoundingAddition = 384;
#if PLATFORM_ANDROID
		if (const FString* MemoryBucketRoundingAdditionVar = FPlatformMisc::GetConfigRulesVariable(TEXT("MemoryBucketRoundingAddition")))
		{
			MemoryBucketRoundingAddition = FCString::Atoi64(**MemoryBucketRoundingAdditionVar);
		}
#endif
		uint32 TotalPhysicalGB = (uint32)((Stats.TotalPhysical + MemoryBucketRoundingAddition * 1024 * 1024 - 1) / 1024 / 1024 / 1024);

		FString HMDRequestedProfileName;
		if (IHeadMountedDisplayModule::IsAvailable())
		{
			HMDRequestedProfileName = IHeadMountedDisplayModule::Get().GetDeviceSystemName();
		}

		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_GPUFamily), GetParam(FAndroidMisc::GetGPUFamily(), TEXT("gpu")));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_GLVersion), FAndroidMisc::GetGLVersion());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_VulkanAvailable), FAndroidMisc::IsVulkanAvailable() ? TEXT("true") : TEXT("false"));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_VulkanVersion), FAndroidMisc::GetVulkanVersion());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_AndroidVersion), FAndroidMisc::GetAndroidVersion());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_DeviceMake), FAndroidMisc::GetDeviceMake());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_DeviceModel), FAndroidMisc::GetDeviceModel());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_DeviceBuildNumber), FAndroidMisc::GetDeviceBuildNumber());
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_UsingHoudini), bUsingHoudini ? TEXT("true") : TEXT("false"));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_Hardware), GetParam(FString(TEXT("unknown")), TEXT("hardware")));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_Chipset), GetParam(FString(TEXT("unknown")), TEXT("chipset")));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_HMDSystemName), HMDRequestedProfileName);
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_TotalPhysicalGB), FString::Printf(TEXT("%d"), TotalPhysicalGB));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_SM5Available), FAndroidMisc::IsDesktopVulkanAvailable() ? TEXT("true") : TEXT("false"));
		AndroidParams.Add(FString(FAndroidProfileSelectorSourceProperties::SRC_VKQuality), FAndroidMisc::GetVKQualityRecommendation());

#if PLATFORM_ANDROID
		// allow ConfigRules to override cvars first
		const TMap<FString, FString>& ConfigRules = FPlatformMisc::GetConfigRuleVars();
		for (const TPair<FString, FString>& Pair : ConfigRules)
		{
			const FString& VariableName = Pair.Key;
			const FString& VariableValue = Pair.Value;
			AndroidParams.Add(FString::Printf(TEXT("SRC_ConfigRuleVar[%s]"), *VariableName), *VariableValue);
		}
#endif
	}
	return AndroidParams;
}

FString const FAndroidDeviceProfileSelectorRuntimeModule::GetRuntimeDeviceProfileName()
{
	static FString ProfileName;

	if (ProfileName.IsEmpty())
	{
		// Fallback profiles in case we do not match any rules
		ProfileName = FPlatformMisc::GetDefaultDeviceProfileName();
		if (ProfileName.IsEmpty())
		{
			ProfileName = FPlatformProperties::PlatformName();
		}
		const TMap<FString, FString>& AndroidParams = GetDeviceSelectorParams();

		FAndroidDeviceProfileSelector::SetSelectorProperties(AndroidParams);

		UE_LOGF(LogAndroid, Log, "Checking %d rules from DeviceProfile ini file.", FAndroidDeviceProfileSelector::GetNumProfiles() );
		UE_LOGF(LogAndroid, Log, "  Default profile: %ls", * ProfileName);
		UE_LOGF(LogAndroid, Log, "  Android selector params: ");
		for(auto& MapIt : AndroidParams)
		{
			UE_LOGF(LogAndroid, Log, "  %ls: %ls", *MapIt.Key, *MapIt.Value);
		}

		const FString& DeviceMake = AndroidParams.FindByHashChecked(GetTypeHash(FAndroidProfileSelectorSourceProperties::SRC_DeviceMake), FAndroidProfileSelectorSourceProperties::SRC_DeviceMake);
		const FString& DeviceModel = AndroidParams.FindByHashChecked(GetTypeHash(FAndroidProfileSelectorSourceProperties::SRC_DeviceModel), FAndroidProfileSelectorSourceProperties::SRC_DeviceModel);

		CheckForJavaSurfaceViewWorkaround(DeviceMake, DeviceModel);
#if !UE_BUILD_SHIPPING

		{
			bool bForceEmulatorProfileSelectionInNonShippingBuilds = true;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bForceEmulatorProfileSelectionInNonShippingBuilds"), bForceEmulatorProfileSelectionInNonShippingBuilds, GEngineIni);
			
			if (bForceEmulatorProfileSelectionInNonShippingBuilds && DeviceMake == TEXT("Google"))
			{
				if (DeviceModel == TEXT("HPE device"))
				{
					ProfileName = TEXT("Android_PC_Emulator");
					UE_LOGF(LogAndroid, Log, "Forced Profile: [Android_PC_Emulator]");
					return ProfileName;
				}
				
				if (AndroidParams.FindByHashChecked(GetTypeHash(FAndroidProfileSelectorSourceProperties::SRC_GPUFamily), FAndroidProfileSelectorSourceProperties::SRC_GPUFamily).StartsWith(TEXT("Android Emulator")))
				{
					ProfileName = TEXT("Android_Emulator");
					UE_LOGF(LogAndroid, Log, "Forced Profile: [Android_Emulator]");
					return ProfileName;
				}
			}
		}
#endif
		
		// Use override from ConfigRules if set
		const FString* ConfigProfile = FPlatformMisc::GetConfigRulesVariable(TEXT("Profile"));
		if (ConfigProfile != nullptr)
		{
			ProfileName = *ConfigProfile;
			UE_LOGF(LogAndroid, Log, "Using ConfigRules Profile: [%ls]", *ProfileName);
		}
		else
		{
			// Find a match with the DeviceProfiles matching rules
			ProfileName = FAndroidDeviceProfileSelector::FindMatchingProfile(ProfileName);
			UE_LOGF(LogAndroid, Log, "Selected Device Profile: [%ls]", *ProfileName);
		}
	}

	return ProfileName;
}

bool FAndroidDeviceProfileSelectorRuntimeModule::GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT)
{
	if (const FString* Found = GetDeviceSelectorParams().Find(PropertyType.ToString()))
	{
		PropertyValueOUT = *Found;
		return true;
	}
	// Special case for non-existent config rule variables
	// they should return true and a value of '[null]'
	// this prevents configrule issues from throwing errors.
	if (PropertyType.ToString().StartsWith(TEXT("SRC_ConfigRuleVar[")))
	{
		PropertyValueOUT = TEXT("[null]");
		return true;
	}

	return false;
}

void FAndroidDeviceProfileSelectorRuntimeModule::CheckForJavaSurfaceViewWorkaround(const FString& DeviceMake, const FString& DeviceModel) const
{
#if USE_ANDROID_JNI
	// We need to initialize the class early as device profiles need to be evaluated before ProcessNewlyLoadedUObjects can be called.
#if !UE_WITH_CONSTINIT_UOBJECT
	extern UClass* Z_Construct_UClass_UAndroidJavaSurfaceViewDevices(ETypeConstructPhase);
	Z_Construct_UClass_UAndroidJavaSurfaceViewDevices(ETypeConstructPhase::Outer);
#endif // !UE_WITH_CONSTINIT_UOBJECT

	const UAndroidJavaSurfaceViewDevices *const SurfaceViewDevices = Cast<UAndroidJavaSurfaceViewDevices>(UAndroidJavaSurfaceViewDevices::StaticClass()->GetDefaultObject());
	check(SurfaceViewDevices);

	for(const FJavaSurfaceViewDevice& Device : SurfaceViewDevices->SurfaceViewDevices)
	{
		if(Device.Manufacturer == DeviceMake && Device.Model == DeviceModel)
		{
			extern void AndroidThunkCpp_UseSurfaceViewWorkaround();
			AndroidThunkCpp_UseSurfaceViewWorkaround();
			return;
		}
	}
#endif
}
