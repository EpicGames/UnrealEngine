// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the Android Device Profile Selector module.
 */
class FAndroidDeviceProfileSelectorRuntimeModule
	: public IDeviceProfileSelectorModule
{
public:

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetRuntimeDeviceProfileName() override;
	virtual bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) override;
	//~ End IDeviceProfileSelectorModule Interface


	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	
	/**
	 * Virtual destructor.
	 */
	virtual ~FAndroidDeviceProfileSelectorRuntimeModule()
	{
	}

private:

	/**
	 * Make sure we use the java workaround for view scaling on devices that require this
	 */
	void CheckForJavaSurfaceViewWorkaround(const FString& DeviceMake, const FString& DeviceModel) const;
};