// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* Dependencies
*****************************************************************************/
#include "Modules/ModuleManager.h"
#include "IDeviceProfileSelectorModule.h"
